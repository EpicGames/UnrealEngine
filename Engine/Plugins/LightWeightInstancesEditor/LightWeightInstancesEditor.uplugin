{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "LightWeightInstancesEditor (DEPRECATED)",
	"Description": "Light Weight Instances provide the flexibility and interaction of actors while having performance similar to instanced meshes. This plugin is now DEPRECATED. Use InstancedActors for similar functionality.",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"DeprecatedEngineVersion": "5.8",
	"Modules": [
		{
			"Name": "LightWeightInstancesEditor",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}