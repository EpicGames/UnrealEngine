// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightWeightInstancesEditor.h"
#include "Modules/ModuleManager.h"
#if WITH_EDITOR
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "LevelEditor.h"
#include "DataLayer/DataLayerEditorSubsystem.h"
#endif
#include "GameFramework/LightWeightInstanceManager.h"
#include "GameFramework/LightWeightInstanceSubsystem.h"

#define LOCTEXT_NAMESPACE "FLightWeightInstancesEditorModule"

DEFINE_LOG_CATEGORY_STATIC(LogLWIEditor, Log, All);

PRAGMA_DISABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR
typedef FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors DelegateType;
#endif

void FLightWeightInstancesEditorModule::StartupModule()
{
	// hook up level editor extension for conversion between light weight instances and actors
	AddLevelViewportMenuExtender();
}

void FLightWeightInstancesEditorModule::ShutdownModule()
{
	// Cleanup menu extenstions
	RemoveLevelViewportMenuExtender();
}

void FLightWeightInstancesEditorModule::AddLevelViewportMenuExtender()
{
#if WITH_EDITOR
	if (!IsRunningGame())
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		auto& MenuExtenders = LevelEditorModule.GetAllLevelViewportContextMenuExtenders();

		MenuExtenders.Add(DelegateType::CreateRaw(this, &FLightWeightInstancesEditorModule::CreateLevelViewportContextMenuExtender));
		LevelViewportExtenderHandle = MenuExtenders.Last().GetHandle();
	}
#endif
}

void FLightWeightInstancesEditorModule::RemoveLevelViewportMenuExtender()
{
#if WITH_EDITOR
	if (LevelViewportExtenderHandle.IsValid())
	{
		FLevelEditorModule* LevelEditorModule = FModuleManager::Get().GetModulePtr<FLevelEditorModule>("LevelEditor");
		if (LevelEditorModule)
		{
			LevelEditorModule->GetAllLevelViewportContextMenuExtenders().RemoveAll([this](const DelegateType& In) { return In.GetHandle() == LevelViewportExtenderHandle; });
		}
	}
#endif
}

#if WITH_EDITOR
TSharedRef<FExtender> FLightWeightInstancesEditorModule::CreateLevelViewportContextMenuExtender(const TSharedRef<FUICommandList> CommandList, const TArray<AActor*> InActors)
{
	TSharedRef<FExtender> Extender = MakeShareable(new FExtender);
	// We only support conversion if all of the actors are the same type
	for (AActor* Actor : InActors)
	{
		if (Actor->GetClass() != InActors[0]->GetClass())
		{
			UE_LOGF(LogLWIEditor, Warning, "Unable to convert actors of multiple types to light weight instances");
			return Extender;
		}
	}

	if (InActors.Num() > 0)
	{
		FText ActorName = InActors.Num() == 1 ? FText::Format(LOCTEXT("ActorNameSingular", "\"{0}\""), FText::FromString(InActors[0]->GetActorLabel())) : LOCTEXT("ActorNamePlural", "Actors");

		FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		TSharedRef<FUICommandList> LevelEditorCommandBindings = LevelEditor.GetGlobalLevelEditorActions();

		// Note: ActorConvert extension point appears only in the pulldown Actor menu.
		Extender->AddMenuExtension("ActorConvert", EExtensionHook::After, LevelEditorCommandBindings, FMenuExtensionDelegate::CreateLambda(
			[this, ActorName, InActors](FMenuBuilder& MenuBuilder) {
				// We can only convert to an LWI if we didn't select an LWI
				const bool bCanExecute = !InActors[0]->GetClass()->IsChildOf(ADEPRECATED_LightWeightInstanceManager::StaticClass());

				MenuBuilder.AddMenuEntry(
					FText::Format(LOCTEXT("ConvertSelectedActorsToLWIsText", "(DEPRECATED) Convert {0} To Light Weight Instances"), ActorName),
					LOCTEXT("ConvertSelectedActorsToLWIsTooltip", "(DEPRECATED) Convert the selected actors to light weight instances."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Convert"),
					FUIAction(
						FExecuteAction::CreateRaw(this, &FLightWeightInstancesEditorModule::ConvertActorsToLWIsUIAction, InActors),
						FCanExecuteAction::CreateLambda([bCanExecute]() { return bCanExecute; }))
				);
			})
		);
	}
	return Extender;
}

void FLightWeightInstancesEditorModule::ConvertActorsToLWIsUIAction(const TArray<AActor*> InActors) const
{
	if (InActors.IsEmpty() || InActors[0] == nullptr)
	{
		UE_LOGF(LogLWIEditor, Log, "Unable to convert unspecified actors to light weight instances");
		return;
	}
	
	for (AActor* Actor : InActors)
	{
		// use the first layer the actor is in if it's in multiple layers
		TArray<const UDataLayerInstance*> DataLayerInstances = Actor->GetDataLayerInstances();
		const UDataLayerInstance* DataLayerInstance = DataLayerInstances.Num() > 0 ? DataLayerInstances[0] : nullptr;

		ADEPRECATED_LightWeightInstanceManager* Manager = FLightWeightInstanceSubsystem::Get().FindOrAddLightWeightInstanceManager(*Actor->GetClass(), *Actor->GetWorld(), Actor->GetActorLocation(), DataLayerInstance);
		check(Manager);
		
		if(UDataLayerEditorSubsystem* DataLayerEditor = UDataLayerEditorSubsystem::Get())
		{
			DataLayerEditor->OnActorDataLayersChanged().Broadcast(Manager);
		}

		Manager->ConvertActorToLightWeightInstance(Actor);
	}
}

#endif // WITH_EDITOR

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FLightWeightInstancesEditorModule, LightWeightInstancesEditor)

