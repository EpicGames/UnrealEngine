// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

#if WITH_EDITOR
class AActor;
class FExtender;
class FUICommandList;
#endif

class FLightWeightInstancesEditorModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

protected:
	// Adds new menu options related to LWIs
	void AddLevelViewportMenuExtender();

	// Cleanup menu options related to LWIs
	void RemoveLevelViewportMenuExtender();

#if WITH_EDITOR
	TSharedRef<FExtender> CreateLevelViewportContextMenuExtender(const TSharedRef<FUICommandList> CommandList, const TArray<AActor*> InActors);

	// Converts InActors to light weight instances. InActors must all be the same type or conversion won't occur
	void ConvertActorsToLWIsUIAction(const TArray<AActor*> InActors) const;

	// Delegates
	FDelegateHandle LevelViewportExtenderHandle;
#endif
};
